// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PSDImporterMaterialDesignerBridge : ModuleRules
{
	public PSDImporterMaterialDesignerBridge(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DynamicMaterial",
				"DynamicMaterialEditor",
				"Engine",
				"ImageCore",
				"ImageWrapper",
				"PSDImporter",
				"PSDImporterCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

		ShortName = "PSImporterMDB";
	}
}
