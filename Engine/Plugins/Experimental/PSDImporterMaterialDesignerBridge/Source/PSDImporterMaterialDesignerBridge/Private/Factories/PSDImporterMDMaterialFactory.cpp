// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterMDMaterialFactory.h"

#include "AssetToolsModule.h"
#include "Components/DMMaterialEffectFunction.h"
#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/DMTextureUV.h"
#include "Components/MaterialStageBlends/DMMSBColor.h"
#include "Components/MaterialStageBlends/DMMSBColorBurn.h"
#include "Components/MaterialStageBlends/DMMSBColorDodge.h"
#include "Components/MaterialStageBlends/DMMSBDarken.h"
#include "Components/MaterialStageBlends/DMMSBDarkenColor.h"
#include "Components/MaterialStageBlends/DMMSBDifference.h"
#include "Components/MaterialStageBlends/DMMSBDivide.h"
#include "Components/MaterialStageBlends/DMMSBExclusion.h"
#include "Components/MaterialStageBlends/DMMSBHardLight.h"
#include "Components/MaterialStageBlends/DMMSBHardMix.h"
#include "Components/MaterialStageBlends/DMMSBHue.h"
#include "Components/MaterialStageBlends/DMMSBLighten.h"
#include "Components/MaterialStageBlends/DMMSBLightenColor.h"
#include "Components/MaterialStageBlends/DMMSBLinearBurn.h"
#include "Components/MaterialStageBlends/DMMSBLinearDodge.h"
#include "Components/MaterialStageBlends/DMMSBLinearLight.h"
#include "Components/MaterialStageBlends/DMMSBLuminosity.h"
#include "Components/MaterialStageBlends/DMMSBMultiply.h"
#include "Components/MaterialStageBlends/DMMSBNormal.h"
#include "Components/MaterialStageBlends/DMMSBOverlay.h"
#include "Components/MaterialStageBlends/DMMSBPinLight.h"
#include "Components/MaterialStageBlends/DMMSBSaturation.h"
#include "Components/MaterialStageBlends/DMMSBScreen.h"
#include "Components/MaterialStageBlends/DMMSBSoftLight.h"
#include "Components/MaterialStageBlends/DMMSBSubtract.h"
#include "Components/MaterialStageBlends/DMMSBVividLight.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSITextureUV.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Materials/Material.h"
#include "Misc/ScopedSlowTask.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "PSDDocument.h"
#include "PSDQuadMeshActor.h"
#include "Utils/DMMaterialSlotFunctionLibrary.h"

#define LOCTEXT_NAMESPACE "PSDImporterMDMaterialFactory"

namespace UE::PSDUIMaterialDesignerBridge::Private
{
	constexpr const TCHAR* MaterialDesignerCropFunctionPath = TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Effects/Alpha/MF_DM_Effect_Alpha_Crop.MF_DM_Effect_Alpha_Crop'");

	UMaterialFunctionInterface* MaterialDesignerCropFunction()
	{
		const TSoftObjectPtr<UMaterialFunctionInterface> MaterialFunctionPtr = TSoftObjectPtr<UMaterialFunctionInterface>(FSoftObjectPath(MaterialDesignerCropFunctionPath));
		return MaterialFunctionPtr.LoadSynchronous();
	}
}

bool UPSDImporterMDMaterialFactory::CanCreateMaterial(const UPSDDocument* InDocument) const
{
	if (!InDocument)
	{
		return false;
	}

	return InDocument->GetTextureCount() <= UE::PSDImporter::MaxSamplerCount;
}

UDynamicMaterialInstance* UPSDImporterMDMaterialFactory::CreateMaterial(UPSDDocument* InDocument) const
{
	if (!InDocument)
	{
		return nullptr;
	}

	FScopedSlowTask SlowTask(2.f, LOCTEXT("CreatingPSDMaterialDesignerMaterial", "Creating PSD Material Designer Material..."));
	SlowTask.MakeDialog();
	SlowTask.EnterProgressFrame(1.f, LOCTEXT("CreatingMaterial", "Creating Material..."));

	UDynamicMaterialInstance* MaterialInstance = CreateMaterialAsset(*InDocument);

	if (!MaterialInstance)
	{
		return nullptr;
	}

	UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(MaterialInstance->GetMaterialModelBase());

	if (!MaterialModel)
	{
		return nullptr;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return nullptr;
	}

	SlowTask.EnterProgressFrame(1.f, LOCTEXT("CreatingLayers", "Creating Layers..."));

	EditorOnlyData->SetDomain(EMaterialDomain::MD_Surface);
	EditorOnlyData->SetBlendMode(EBlendMode::BLEND_Translucent);
	EditorOnlyData->SetShadingModel(EDMMaterialShadingModel::Unlit);

	CreateLayers(*EditorOnlyData, *InDocument);

	return MaterialInstance;
}

UDynamicMaterialInstance* UPSDImporterMDMaterialFactory::CreateMaterialAsset(UPSDDocument& InDocument) const
{
	const FString BasePath = FPaths::GetPath(InDocument.GetPackage()->GetPathName());
	FString AssetName = InDocument.GetName();

	if (AssetName.StartsWith(TEXT("PSD_")))
	{
		AssetName = TEXT("MD_") + AssetName.RightChop(4);
	}
	else
	{
		AssetName = TEXT("MD_") + AssetName;
	}

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	FString PackageName;
	AssetTools.CreateUniqueAssetName(BasePath / AssetName, TEXT(""), PackageName, AssetName);

	return Cast<UDynamicMaterialInstance>(AssetTools.CreateAsset(
		AssetName, 
		FPaths::GetPath(PackageName), 
		UDynamicMaterialInstance::StaticClass(), 
		NewObject<UDynamicMaterialInstanceFactory>()
	));
}

void UPSDImporterMDMaterialFactory::CreateLayers(UDynamicMaterialModelEditorOnlyData& InEditorOnlyData, UPSDDocument& InDocument) const
{
	UDMMaterialSlot* EmissiveSlot = InEditorOnlyData.AddSlotForMaterialProperty(EDMMaterialPropertyType::EmissiveColor);

	if (!EmissiveSlot)
	{
		return;
	}

	bool bIsFirstLayer = true;
	TArray<const FPSDFileLayer*> ValidLayers = InDocument.GetValidLayers();

	for (const FPSDFileLayer* PSDLayer : ValidLayers)
	{
		CreateLayer(*EmissiveSlot, InDocument, *PSDLayer, bIsFirstLayer);
		bIsFirstLayer = false;
	}
}

void UPSDImporterMDMaterialFactory::CreateLayer(UDMMaterialSlot& InSlot, UPSDDocument& InDocument, const FPSDFileLayer& InLayer,
	bool bInIsFirstLayer) const
{
	UDMMaterialLayerObject* MaterialLayer = UDMMaterialSlotFunctionLibrary::AddTextureLayer(
		&InSlot,
		InLayer.Texture.LoadSynchronous(),
		EDMMaterialPropertyType::EmissiveColor,
		/* Replace Slot */ bInIsFirstLayer
	);

	if (!MaterialLayer)
	{
		return;
	}

	CreateLayer_Base(*MaterialLayer, InLayer);

	const FIntPoint& DocumentSize = InDocument.GetSize();

	if (DocumentSize.X <= 0 || DocumentSize.Y <= 0)
	{
		return;
	}

	const FIntRect& LayerBounds = InLayer.Bounds;

	if (LayerBounds.Width() <= 0 || LayerBounds.Height() <= 0)
	{
		return;
	}

	if (!InLayer.IsLayerFullSize(DocumentSize))
	{
		CreateLayer_Base_Crop(*MaterialLayer, InLayer, DocumentSize);
	}

	const FIntRect& MaskBounds = InLayer.MaskBounds;

	if (!InLayer.HasMask() || MaskBounds.Width() <= 0 || MaskBounds.Height() <= 0)
	{
		CreateLayer_Mask_None(*MaterialLayer);
	}
	else
	{
		CreateLayer_Mask(*MaterialLayer, InLayer);

		if (!InLayer.IsLayerFullSize(DocumentSize) || LayerBounds != MaskBounds)
		{
			CreateLayer_Mask_Crop(*MaterialLayer, InLayer, DocumentSize);
		}
	}

	MaterialLayer->Update(MaterialLayer, EDMUpdateType::Structure);
}

void UPSDImporterMDMaterialFactory::CreateLayer_Base(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer) const
{
	UDMMaterialStage* BaseStage = InMaterialLayer.GetStage(EDMMaterialLayerStage::Base);

	if (!ensure(BaseStage))
	{
		return;
	}

	UClass* BlendModeForLayer = GetMaterialDesignerBlendMode(InLayer.BlendMode);

	if (BlendModeForLayer && BlendModeForLayer != UDMMaterialStageBlendNormal::StaticClass())
	{
		UDMMaterialStageBlend* Blend = NewObject<UDMMaterialStageBlend>(BaseStage, BlendModeForLayer, NAME_None, RF_Transactional);
		BaseStage->SetSource(Blend);
	}

	UDMMaterialStageInputExpression* NewExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		BaseStage,
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (!ensure(NewExpression))
	{
		return;
	}

	if (UDMMaterialStageExpressionTextureSampleBase* TextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(NewExpression->GetMaterialStageExpression()))
	{
		TextureSample->SetClampTextureEnabled(true);
	}

	UDMMaterialSubStage* BaseSubStage = NewExpression->GetSubStage();

	if (!ensure(BaseSubStage))
	{
		return;
	}

	UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		BaseSubStage,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		EDMValueType::VT_Texture,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (ensure(InputValue))
	{
		UDMMaterialValueTexture* InputTexture = Cast<UDMMaterialValueTexture>(InputValue->GetValue());

		if (ensure(InputTexture))
		{
			InputTexture->SetDefaultValue(InLayer.Texture.LoadSynchronous());
			InputTexture->ApplyDefaultValue();
		}
	}
}

void UPSDImporterMDMaterialFactory::CreateLayer_Base_Crop(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer, 
	const FIntPoint& InDocumentSize) const
{
	UDMMaterialStage* BaseStage = InMaterialLayer.GetStage(EDMMaterialLayerStage::Base);

	if (!BaseStage)
	{
		return;
	}

	const FIntRect& Bounds = InLayer.Bounds;

	const float BoundsWidth = Bounds.Width();
	const float BoundsHeight = Bounds.Height();

	const float OffsetLeft = static_cast<float>(Bounds.Min.X) / static_cast<float>(BoundsWidth);
	const float OffsetTop = static_cast<float>(Bounds.Min.Y) / static_cast<float>(BoundsHeight);

	const float ScaleHorizontal = static_cast<float>(InDocumentSize.X) / static_cast<float>(BoundsWidth);
	const float ScaleVertical = static_cast<float>(InDocumentSize.Y) / static_cast<float>(BoundsHeight);

	// Update UVs
	for (UDMMaterialStageInput* BaseInput : BaseStage->GetInputs())
	{
		if (UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(BaseInput))
		{
			if (UDMMaterialSubStage* SubStage = InputExpression->GetSubStage())
			{
				for (UDMMaterialStageInput* SubInput : SubStage->GetInputs())
				{
					if (UDMMaterialStageInputTextureUV* InputTextureUV = Cast<UDMMaterialStageInputTextureUV>(SubInput))
					{
						if (UDMTextureUV* TextureUV = InputTextureUV->GetTextureUV())
						{
							TextureUV->SetOffset({OffsetLeft, -OffsetTop});
							TextureUV->SetTiling({ScaleHorizontal, ScaleVertical});
							TextureUV->SetPivot(FVector2D::ZeroVector);
						}

						break;
					}
				}
			}
		}
	}

	// If we don't have a mask, we need to crop the alpha based on the base layer.
	if (InLayer.HasMask() && !InLayer.MaskBounds.IsEmpty())
	{
		return;
	}

	// Create crop stack
	UDMMaterialEffectStack* EffectStack = InMaterialLayer.GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	UMaterialFunctionInterface* CropFunction = UE::PSDUIMaterialDesignerBridge::Private::MaterialDesignerCropFunction();

	if (!CropFunction)
	{
		return;
	}

	UDMMaterialEffectFunction* CropEffect = UDMMaterialEffect::CreateEffect<UDMMaterialEffectFunction>(EffectStack);
	CropEffect->SetMaterialFunction(CropFunction);

	const TArray<TObjectPtr<UDMMaterialValue>>& Inputs = CropEffect->GetInputValues();

	if (Inputs.Num() != 6)
	{
		return;
	}

	// 0 In R
	// 10 Crop Right 0-1
	// 11 Crop Left 0-1
	// 15 Crop Top 0-1
	// 17 Crop Bottom 0-1
	// 99 Amount 0-100

	const float CropLeft = static_cast<float>(Bounds.Min.X) / static_cast<float>(InDocumentSize.X);
	const float CropRight = 1.f - (static_cast<float>(Bounds.Max.X) / static_cast<float>(InDocumentSize.X));
	const float CropTop = static_cast<float>(Bounds.Min.Y) / static_cast<float>(InDocumentSize.Y);
	const float CropBottom = 1.f - (static_cast<float>(Bounds.Max.Y) / static_cast<float>(InDocumentSize.Y));

	UDMMaterialValue* InputIn = Inputs[0];
	UDMMaterialValueFloat1* InputCropRight = Cast<UDMMaterialValueFloat1>(Inputs[1]);
	UDMMaterialValueFloat1* InputCropLeft = Cast<UDMMaterialValueFloat1>(Inputs[2]);
	UDMMaterialValueFloat1* InputCropTop = Cast<UDMMaterialValueFloat1>(Inputs[3]);
	UDMMaterialValueFloat1* InputCropBottom = Cast<UDMMaterialValueFloat1>(Inputs[4]);
	UDMMaterialValueFloat1* InputAmount = Cast<UDMMaterialValueFloat1>(Inputs[5]);

	// InputIn should be nullptr
	if (InputIn || !InputCropRight || !InputCropLeft || !InputCropTop || !InputCropBottom || !InputAmount)
	{
		return;
	}

	InputCropLeft->SetDefaultValue(FMath::Clamp(CropLeft, 0.f, 1.f));
	InputCropLeft->ApplyDefaultValue();

	InputCropRight->SetDefaultValue(FMath::Clamp(CropRight, 0.f, 1.f));
	InputCropRight->ApplyDefaultValue();

	InputCropTop->SetDefaultValue(FMath::Clamp(CropTop, 0.f, 1.f));
	InputCropTop->ApplyDefaultValue();

	InputCropBottom->SetDefaultValue(FMath::Clamp(CropBottom, 0.f, 1.f));
	InputCropBottom->ApplyDefaultValue();

	InputAmount->SetDefaultValue(100.f);
	InputAmount->ApplyDefaultValue();

	EffectStack->AddEffect(CropEffect);
}

void UPSDImporterMDMaterialFactory::CreateLayer_Mask_None(UDMMaterialLayerObject& InMaterialLayer) const
{
	UDMMaterialStage* MaskStage = InMaterialLayer.GetStage(EDMMaterialLayerStage::Mask);

	if (!ensure(MaskStage))
	{
		return;
	}

	if (UDMMaterialStageThroughputLayerBlend* LayerBlend = Cast<UDMMaterialStageThroughputLayerBlend>(MaskStage->GetSource()))
	{
		if (UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(LayerBlend->GetInputMask()))
		{
			if (UDMMaterialStageExpressionTextureSample* TextureSample = Cast<UDMMaterialStageExpressionTextureSample>(InputExpression->GetMaterialStageExpression()))
			{
				TextureSample->SetUseBaseTexture(true);
			}
		}

		LayerBlend->SetMaskChannelOverride(EAvaColorChannel::Alpha);
	}
}

void UPSDImporterMDMaterialFactory::CreateLayer_Mask(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer) const
{
	UDMMaterialStage* MaskStage = InMaterialLayer.GetStage(EDMMaterialLayerStage::Mask);

	if (!ensure(MaskStage))
	{
		return;
	}

	UDMMaterialStageInputExpression* NewExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		MaskStage,
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		UDMMaterialStageThroughputLayerBlend::InputMaskSource,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (!ensure(NewExpression))
	{
		return;
	}

	if (UDMMaterialStageExpressionTextureSampleBase* TextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(NewExpression->GetMaterialStageExpression()))
	{
		TextureSample->SetClampTextureEnabled(true);
	}

	UDMMaterialSubStage* MaskSubStage = NewExpression->GetSubStage();

	if (!ensure(MaskSubStage))
	{
		return;
	}

	UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		MaskSubStage,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		EDMValueType::VT_Texture,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (ensure(InputValue))
	{
		UDMMaterialValueTexture* InputTexture = Cast<UDMMaterialValueTexture>(InputValue->GetValue());

		if (ensure(InputTexture))
		{
			InputTexture->SetValue(InLayer.Mask.LoadSynchronous());
		}
	}
}

void UPSDImporterMDMaterialFactory::CreateLayer_Mask_Crop(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer,
	const FIntPoint& InDocumentSize) const
{
	UDMMaterialStage* MaskStage = InMaterialLayer.GetStage(EDMMaterialLayerStage::Mask);

	if (!MaskStage)
	{
		return;
	}

	const FIntRect& LayerBounds = InLayer.Bounds;
	const FIntRect& MaskBounds = InLayer.MaskBounds;

	const float MaskBoundsWidth = MaskBounds.Width();
	const float MaskBoundsHeight = MaskBounds.Height();

	const float MaskOffsetLeft = static_cast<float>(MaskBounds.Min.X) / static_cast<float>(MaskBoundsWidth);
	const float MaskOffsetTop = static_cast<float>(MaskBounds.Min.Y) / static_cast<float>(MaskBoundsHeight);

	const float MaskScaleHorizontal = static_cast<float>(InDocumentSize.X) / static_cast<float>(MaskBoundsWidth);
	const float MaskScaleVertical = static_cast<float>(InDocumentSize.Y) / static_cast<float>(MaskBoundsHeight);

	// Update UVs
	for (UDMMaterialStageInput* MaskInput : MaskStage->GetInputs())
	{
		if (UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(MaskInput))
		{
			if (UDMMaterialSubStage* SubStage = InputExpression->GetSubStage())
			{
				for (UDMMaterialStageInput* SubInput : SubStage->GetInputs())
				{
					if (UDMMaterialStageInputTextureUV* InputTextureUV = Cast<UDMMaterialStageInputTextureUV>(SubInput))
					{
						if (UDMTextureUV* TextureUV = InputTextureUV->GetTextureUV())
						{
							TextureUV->SetOffset({MaskOffsetLeft, -MaskOffsetTop});
							TextureUV->SetTiling({MaskScaleHorizontal, MaskScaleVertical});
							TextureUV->SetPivot(FVector2D::ZeroVector);
						}

						break;
					}
				}
			}
		}
	}

	// Create crop stack
	UDMMaterialEffectStack* EffectStack = InMaterialLayer.GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	UMaterialFunctionInterface* CropFunction = UE::PSDUIMaterialDesignerBridge::Private::MaterialDesignerCropFunction();

	if (!CropFunction)
	{
		return;
	}

	UDMMaterialEffectFunction* CropEffect = UDMMaterialEffect::CreateEffect<UDMMaterialEffectFunction>(EffectStack);
	CropEffect->SetMaterialFunction(CropFunction);

	const TArray<TObjectPtr<UDMMaterialValue>>& Inputs = CropEffect->GetInputValues();

	if (Inputs.Num() != 6)
	{
		return;
	}

	// 0 In R
	// 10 Crop Right 0-1
	// 11 Crop Left 0-1
	// 15 Crop Top 0-1
	// 17 Crop Bottom 0-1
	// 99 Amount 0-100

	const float CropLeft = static_cast<float>(FMath::Max(LayerBounds.Min.X, MaskBounds.Min.X)) / static_cast<float>(InDocumentSize.X);
	const float CropRight = 1.f - (static_cast<float>(FMath::Min(LayerBounds.Max.X, MaskBounds.Max.X)) / static_cast<float>(InDocumentSize.X));
	const float CropTop = static_cast<float>(FMath::Max(LayerBounds.Min.Y, MaskBounds.Min.Y)) / static_cast<float>(InDocumentSize.Y);
	const float CropBottom = 1.f - (static_cast<float>(FMath::Min(LayerBounds.Max.Y, MaskBounds.Max.Y)) / static_cast<float>(InDocumentSize.Y));

	UDMMaterialValue* InputIn = Inputs[0];
	UDMMaterialValueFloat1* InputCropRight = Cast<UDMMaterialValueFloat1>(Inputs[1]);
	UDMMaterialValueFloat1* InputCropLeft = Cast<UDMMaterialValueFloat1>(Inputs[2]);
	UDMMaterialValueFloat1* InputCropTop = Cast<UDMMaterialValueFloat1>(Inputs[3]);
	UDMMaterialValueFloat1* InputCropBottom = Cast<UDMMaterialValueFloat1>(Inputs[4]);
	UDMMaterialValueFloat1* InputAmount = Cast<UDMMaterialValueFloat1>(Inputs[5]);

	// InputIn should be nullptr
	if (InputIn || !InputCropRight || !InputCropLeft || !InputCropTop || !InputCropBottom || !InputAmount)
	{
		return;
	}

	InputCropLeft->SetDefaultValue(FMath::Clamp(CropLeft, 0.f, 1.f));
	InputCropLeft->ApplyDefaultValue();

	InputCropRight->SetDefaultValue(FMath::Clamp(CropRight, 0.f, 1.f));
	InputCropRight->ApplyDefaultValue();

	InputCropTop->SetDefaultValue(FMath::Clamp(CropTop, 0.f, 1.f));
	InputCropTop->ApplyDefaultValue();

	InputCropBottom->SetDefaultValue(FMath::Clamp(CropBottom, 0.f, 1.f));
	InputCropBottom->ApplyDefaultValue();

	InputAmount->SetDefaultValue(100.f);
	InputAmount->ApplyDefaultValue();

	EffectStack->AddEffect(CropEffect);

	InMaterialLayer.SetTextureUVLinkEnabled(false);
}

UClass* UPSDImporterMDMaterialFactory::GetMaterialDesignerBlendMode(EPSDBlendMode InBlendMode) const
{
	switch (InBlendMode)
	{
		default:
		case EPSDBlendMode::PassThrough:
		case EPSDBlendMode::Dissolve:
			return nullptr; // Not supported

		case EPSDBlendMode::Normal:
			return UDMMaterialStageBlendNormal::StaticClass();

		case EPSDBlendMode::Darken:
			return UDMMaterialStageBlendDarken::StaticClass();

		case EPSDBlendMode::Multiply:
			return UDMMaterialStageBlendMultiply::StaticClass();

		case EPSDBlendMode::ColorBurn:
			return UDMMaterialStageBlendColorBurn::StaticClass();

		case EPSDBlendMode::LinearBurn:
			return UDMMaterialStageBlendLinearBurn::StaticClass();

		case EPSDBlendMode::DarkerColor:
			return UDMMaterialStageBlendDarkenColor::StaticClass();

		case EPSDBlendMode::Lighten:
			return UDMMaterialStageBlendLighten::StaticClass();

		case EPSDBlendMode::Screen:
			return UDMMaterialStageBlendScreen::StaticClass();

		case EPSDBlendMode::ColorDodge:
			return UDMMaterialStageBlendColorDodge::StaticClass();

		case EPSDBlendMode::LinearDodge:
			return UDMMaterialStageBlendLinearDodge::StaticClass();

		case EPSDBlendMode::LighterColor:
			return UDMMaterialStageBlendLightenColor::StaticClass();

		case EPSDBlendMode::Overlay:
			return UDMMaterialStageBlendOverlay::StaticClass();

		case EPSDBlendMode::SoftLight:
			return UDMMaterialStageBlendSoftLight::StaticClass();

		case EPSDBlendMode::HardLight:
			return UDMMaterialStageBlendHardLight::StaticClass();

		case EPSDBlendMode::VividLight:
			return UDMMaterialStageBlendVividLight::StaticClass();

		case EPSDBlendMode::LinearLight:
			return UDMMaterialStageBlendLinearLight::StaticClass();

		case EPSDBlendMode::PinLight:
			return UDMMaterialStageBlendPinLight::StaticClass();

		case EPSDBlendMode::HardMix:
			return UDMMaterialStageBlendHardMix::StaticClass();

		case EPSDBlendMode::Difference:
			return UDMMaterialStageBlendDifference::StaticClass();

		case EPSDBlendMode::Exclusion:
			return UDMMaterialStageBlendExclusion::StaticClass();

		case EPSDBlendMode::Subtract:
			return UDMMaterialStageBlendSubtract::StaticClass();

		case EPSDBlendMode::Divide:
			return UDMMaterialStageBlendDivide::StaticClass();

		case EPSDBlendMode::Hue:
			return UDMMaterialStageBlendHue::StaticClass();

		case EPSDBlendMode::Saturation:
			return UDMMaterialStageBlendSaturation::StaticClass();

		case EPSDBlendMode::Color:
			return UDMMaterialStageBlendColor::StaticClass();

		case EPSDBlendMode::Luminosity:
			return UDMMaterialStageBlendLuminosity::StaticClass();
	}
}

#undef LOCTEXT_NAMESPACE
