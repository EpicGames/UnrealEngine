// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Math/MathFwd.h"

#include "PSDImporterMDMaterialFactory.generated.h"

class UClass;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;
class UDynamicMaterialModelEditorOnlyData;
class UPSDDocument;
enum class EPSDBlendMode : uint8;
struct FPSDFileLayer;

UCLASS()
class UPSDImporterMDMaterialFactory : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UPSDImporterMDMaterialFactory() override = default;

	bool CanCreateMaterial(const UPSDDocument* InDocument) const;

	UDynamicMaterialInstance* CreateMaterial(UPSDDocument* InDocument) const;

protected:
	UDynamicMaterialInstance* CreateMaterialAsset(UPSDDocument& InDocument) const;

	void CreateLayers(UDynamicMaterialModelEditorOnlyData& InEditorOnlyData, UPSDDocument& InDocument) const;

	void CreateLayer(UDMMaterialSlot& InSlot, UPSDDocument& InDocument, const FPSDFileLayer& InLayer, bool bInIsFirstLayer) const;

	void CreateLayer_Base(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer) const;
	void CreateLayer_Base_Crop(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer, const FIntPoint& InDocumentSize) const;

	void CreateLayer_Mask_None(UDMMaterialLayerObject& InMaterialLayer) const;

	void CreateLayer_Mask(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer) const;
	void CreateLayer_Mask_Crop(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer, const FIntPoint& InDocumentSize) const;

	void CreateLayer_Crop(UDMMaterialLayerObject& InMaterialLayer, const FPSDFileLayer& InLayer, const FIntPoint& InDocumentSize,
		UDMMaterialStage* InStage, const FIntRect& InBounds) const;

	UClass* GetMaterialDesignerBlendMode(EPSDBlendMode InBlendMode) const;
};
