// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterMDQuadsFactory.h"

#include "Engine/World.h"
#include "Materials/Material.h"
#include "PSDDocument.h"
#include "PSDQuadActor.h"
#include "PSDQuadMeshActor.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Misc/ScopedSlowTask.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"

#define LOCTEXT_NAMESPACE "PSDImporterMDQuads"

namespace UE::PSDUIMaterialDesignerBridge::Private
{
	constexpr const TCHAR* MaterialDesignerTemplatePath = TEXT("/Script/DynamicMaterial.DynamicMaterialInstance'/PSDImporterMaterialDesignerBridge/PSDImporterMaterialDesignerBridge/MD_PSDImporter_Quad.MD_PSDImporter_Quad'");
	constexpr const TCHAR* MaterialDesignerTemplatePath_Mask_NoCrop = TEXT("/Script/DynamicMaterial.DynamicMaterialInstance'/PSDImporterMaterialDesignerBridge/PSDImporterMaterialDesignerBridge/MD_PSDImporter_Quad_Mask_NoCrop.MD_PSDImporter_Quad_Mask_NoCrop'");
	constexpr const TCHAR* MaterialDesignerTemplatePath_Mask_Crop = TEXT("/Script/DynamicMaterial.DynamicMaterialInstance'/PSDImporterMaterialDesignerBridge/PSDImporterMaterialDesignerBridge/MD_PSDImporter_Quad_Mask_Crop.MD_PSDImporter_Quad_Mask_Crop'");

	UDynamicMaterialInstance* GetMaterialDesignerTemplate(const TCHAR* InPath)
	{
		const TSoftObjectPtr<UDynamicMaterialInstance> MaterialPtr = TSoftObjectPtr<UDynamicMaterialInstance>(FSoftObjectPath(InPath));
		return MaterialPtr.LoadSynchronous();
	}
}

APSDQuadActor* UPSDImporterMDQuadsFactory::CreateQuadActor(UWorld& InWorld, UPSDDocument& InDocument) const
{
	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.bNoFail = true;

	APSDQuadActor* QuadActor = InWorld.SpawnActor<APSDQuadActor>(SpawnParams);
	QuadActor->SetPSDDocument(InDocument);
	QuadActor->SetActorScale3D(FVector(0.1, 0.1, 0.1));

	return QuadActor;
}

void UPSDImporterMDQuadsFactory::CreateQuads(APSDQuadActor& InQuadActor, EPSDImporterMaterialDesignerType InType) const
{
	UPSDDocument* Document = InQuadActor.GetPSDDocument();

	if (!Document)
	{
		return;
	}

	const TArray<FPSDFileLayer>& Layers = Document->GetLayers();

	const FText QuadPrompt = LOCTEXT("CreatingPSDMaterialDesignerQuads", "Creating PSD Material Designer Quads...");
	FScopedSlowTask SlowTask(Layers.Num(), QuadPrompt);
	SlowTask.MakeDialog();

	for (int32 Index = 0; Index < Layers.Num(); ++Index)
	{
		SlowTask.EnterProgressFrame(1.f, QuadPrompt);

		if (!Layers[Index].bIsVisible || Layers[Index].Clipping != 0 || !Layers[Index].bIsSupportedLayerType
			|| FMath::IsNearlyZero(Layers[Index].Opacity)
			|| (Layers[Index].Bounds.Area() == 0))
		{
			continue;
		}

		if (APSDQuadMeshActor* QuadMesh = CreateQuad(InQuadActor, Index, InType))
		{
			InQuadActor.AddQuadMesh(*QuadMesh);
		}
	}

	InQuadActor.InitComplete();
}

APSDQuadMeshActor* UPSDImporterMDQuadsFactory::CreateQuad(APSDQuadActor& InQuadActor, int32 InLayerIndex, 
	EPSDImporterMaterialDesignerType InType) const
{
	using namespace UE::PSDUIMaterialDesignerBridge::Private;

	UPSDDocument* Document = InQuadActor.GetPSDDocument();

	if (!Document)
	{
		return nullptr;
	}

	if (!Document->GetLayers().IsValidIndex(InLayerIndex))
	{
		return nullptr;
	}

	const FPSDFileLayer& Layer = Document->GetLayers()[InLayerIndex];
	const TCHAR* BaseMaterialPath = nullptr;
	const FIntPoint DocumentSize = Document->GetSize();

	if (!Layer.NeedsCrop(DocumentSize))
	{
		if (Layer.HasMask())
		{
			BaseMaterialPath = MaterialDesignerTemplatePath_Mask_NoCrop;
		}
		else
		{
			BaseMaterialPath = MaterialDesignerTemplatePath;
		}
	}
	else
	{
		BaseMaterialPath = MaterialDesignerTemplatePath_Mask_Crop;
	}

	UWorld* World = InQuadActor.GetWorld();

	if (!World)
	{
		return nullptr;
	}

	FActorSpawnParameters Params;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	Params.bNoFail = true;

	if (InQuadActor.bIsEditorPreviewActor)
	{
		Params.bTemporaryEditorActor = true;
		Params.bHideFromSceneOutliner = true;
	}

	APSDQuadMeshActor* QuadMesh = World->SpawnActor<APSDQuadMeshActor>(Params);

	if (!QuadMesh)
	{
		return nullptr;
	}

	UDynamicMaterialInstance* Template = GetMaterialDesignerTemplate(BaseMaterialPath);

	if (!Template)
	{
		QuadMesh->Destroy();
		return nullptr;
	}

	UDynamicMaterialModel* TemplateModel = Template->GetMaterialModel();

	if (!TemplateModel)
	{
		QuadMesh->Destroy();
		return nullptr;
	}

	UDynamicMaterialInstanceFactory* MaterialInstanceFactory = NewObject<UDynamicMaterialInstanceFactory>();
	UDynamicMaterialInstance* LayerMaterial = nullptr;

	switch (InType)
	{
		case EPSDImporterMaterialDesignerType::Instance:
		{
			UDynamicMaterialModelDynamic* InstanceModel = UDynamicMaterialModelDynamic::Create(QuadMesh, TemplateModel);

			if (!InstanceModel)
			{
				QuadMesh->Destroy();
				return nullptr;
			}

			LayerMaterial = Cast<UDynamicMaterialInstance>(MaterialInstanceFactory->FactoryCreateNew(
				UDynamicMaterialInstance::StaticClass(),
				QuadMesh,
				NAME_None,
				RF_Transactional,
				InstanceModel,
				GWarn
			));

			break;
		}

		case EPSDImporterMaterialDesignerType::Copy:
		{
			LayerMaterial = Cast<UDynamicMaterialInstance>(MaterialInstanceFactory->FactoryCreateNew(
				UDynamicMaterialInstance::StaticClass(),
				QuadMesh,
				NAME_None,
				RF_Transactional,
				nullptr,
				GWarn
			));

			UDMMaterialModelFunctionLibrary::DuplicateModelBetweenMaterials(TemplateModel, LayerMaterial);
			break;
		}

		default:
			QuadMesh->Destroy();
			return nullptr;
	}

	QuadMesh->InitLayer(InQuadActor, InLayerIndex, LayerMaterial);

	if (InQuadActor.bIsEditorPreviewActor)
	{
		QuadMesh->SetActorEnableCollision(false);
	}

	return QuadMesh;
}

#undef LOCTEXT_NAMESPACE
