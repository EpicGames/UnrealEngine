// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "PSDImporterMDQuadsFactory.generated.h"

class APSDQuadActor;
class APSDQuadMeshActor;
class UPSDDocument;
class UWorld;

enum class EPSDImporterMaterialDesignerType : uint8
{
	Instance,
	Copy
};

UCLASS()
class UPSDImporterMDQuadsFactory : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UPSDImporterMDQuadsFactory() override = default;

	APSDQuadActor* CreateQuadActor(UWorld& InWorld, UPSDDocument& InDocument) const;

	void CreateQuads(APSDQuadActor& InQuadActor, EPSDImporterMaterialDesignerType InType) const;

protected:
	APSDQuadMeshActor* CreateQuad(APSDQuadActor& InQuadActor, int32 InLayerIndex, EPSDImporterMaterialDesignerType InType) const;
};
