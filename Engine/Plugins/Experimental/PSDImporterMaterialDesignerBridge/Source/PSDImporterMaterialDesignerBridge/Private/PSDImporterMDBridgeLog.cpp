// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterMDBridgeLog.h"

DEFINE_LOG_CATEGORY(LogPSDImporterMaterialDesignerBridge);
