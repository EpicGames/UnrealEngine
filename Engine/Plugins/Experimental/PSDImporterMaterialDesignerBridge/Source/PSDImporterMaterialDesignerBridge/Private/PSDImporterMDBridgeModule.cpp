// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterMDBridgeModule.h"

#include "Engine/Texture.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"
#include "PSDQuadMeshActor.h"
#include "PSDImporterMDBridgeUtils.h"
#include "PSDImporterMDContentBrowserIntegration.h"

void FPSDImporterMaterialDesignerBridgeModule::StartupModule()
{
	FPSDImporterMaterialDesignerContentBrowserIntegration::Get().Integrate();

	if (!TextureResetDelegate.IsValid())
	{
		TextureResetDelegate = APSDQuadMeshActor::GetTextureResetDelegate().AddStatic(&FPSDImporterMDBridgeUtils::ResetTexture);
	}
}
	
void FPSDImporterMaterialDesignerBridgeModule::ShutdownModule()
{
	FPSDImporterMaterialDesignerContentBrowserIntegration::Get().Disintegrate();

	if (TextureResetDelegate.IsValid())
	{
		APSDQuadMeshActor::GetTextureResetDelegate().Remove(TextureResetDelegate);
		TextureResetDelegate.Reset();
	}
}

IMPLEMENT_MODULE(FPSDImporterMaterialDesignerBridgeModule, PSDImporterMaterialDesignerBridge);
