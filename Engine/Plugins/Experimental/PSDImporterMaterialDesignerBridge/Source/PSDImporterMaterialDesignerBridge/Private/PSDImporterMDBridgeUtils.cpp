// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterMDBridgeUtils.h"

#include "Components/DMMaterialValue.h"
#include "Components/DMTextureUV.h"
#include "Components/DMTextureUVDynamic.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Components/MaterialValuesDynamic/DMMaterialValueFloat1Dynamic.h"
#include "Components/MaterialValuesDynamic/DMMaterialValueTextureDynamic.h"
#include "Engine/Texture.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "PSDDocument.h"
#include "PSDFile.h"
#include "PSDImporterMaterialDesignerBridge/Public/PSDImporterMDConstants.h"
#include "PSDQuadActor.h"
#include "PSDQuadMeshActor.h"

namespace UE::PSDImporterMaterialDesignerBridge::Private
{
	template<typename InValueType>
	void SetTextureValue(InValueType* InValue, const FName& InParameterName, UTexture* InLayerTexture, UTexture* InMaskTexture)
	{
		if (InParameterName == TextureEmissiveParameterName)
		{
			InValue->SetValue(InLayerTexture);
		}
		else if (InParameterName == TextureOpacityParameterName)
		{
			if (InMaskTexture)
			{
				InValue->SetValue(InMaskTexture);
			}
			else
			{
				InValue->ApplyDefaultValue();
			}
		}
	}

	FVector4 CalculateMaskParams(const FPSDFileLayer& InLayer)
	{
		const FIntRect LayerBounds = InLayer.Bounds;
		const FIntRect MaskBounds = InLayer.MaskBounds;
		const FVector2D LayerSizeFloat = FVector2D(LayerBounds.Size());
		const FVector2D MaskSizeFloat = FVector2D(MaskBounds.Size());
		const FVector2D Offset = FVector2D(MaskBounds.Min - LayerBounds.Min) / MaskSizeFloat;
		const FVector2D Tiling = LayerSizeFloat / MaskSizeFloat; // Correct
		return FVector4(Offset.X, Offset.Y, Tiling.X, Tiling.Y);
	}

	FVector4 CalculateCropParams(const FPSDFileLayer& InLayer)
	{
		const FIntRect LayerBounds = InLayer.Bounds;
		const FIntRect MaskBounds = InLayer.MaskBounds;
		const FVector2D LayerSizeFloat = FVector2D(LayerBounds.Size());
		const FVector2D Min = FVector2D(MaskBounds.Min - LayerBounds.Min) / LayerSizeFloat;
		const FVector2D Max = FVector2D(MaskBounds.Max - LayerBounds.Min) / LayerSizeFloat;
		return FVector4(Min.X, Min.Y, Max.X, Max.Y);
	}

	template<typename InValueType>
	void SetTextureUV(InValueType* InValue, const FVector4 InUVParams)
	{
		InValue->SetOffset({InUVParams.X, -InUVParams.Y});
		InValue->SetTiling({InUVParams.Z, InUVParams.W}); // Correct
		InValue->SetPivot(FVector2D::ZeroVector);
		InValue->SetRotation(0.f);
	}

	template<typename InValueType>
	void SetCropValue(InValueType* InValue, const FName& InParameterName, const FVector4 InCropParams)
	{
		if (InParameterName == OpacityCropLeft)
		{
			InValue->SetValue(FMath::Clamp(InCropParams.X, 0.f, 1.f));
		}
		else if (InParameterName == OpacityCropTop)
		{
			InValue->SetValue(FMath::Clamp(InCropParams.Y, 0.f, 1.f));
		}
		else if (InParameterName == OpacityCropRight)
		{
			InValue->SetValue(FMath::Clamp(1.f - InCropParams.Z, 0.f, 1.f));
		}
		else if (InParameterName == OpacityCropBottom)
		{
			InValue->SetValue(FMath::Clamp(1.f - InCropParams.W, 0.f, 1.f));
		}
	}
}



void FPSDImporterMDBridgeUtils::ResetTexture(APSDQuadMeshActor& InQuadMeshActor)
{
	using namespace UE::PSDImporterMaterialDesignerBridge::Private;

	APSDQuadActor* QuadActor = InQuadMeshActor.GetQuadActor();

	if (!QuadActor)
	{
		return;
	}

	UPSDDocument* PSDDocument = QuadActor->GetPSDDocument();

	if (!PSDDocument)
	{
		return;
	}

	UMaterialInterface* Material = InQuadMeshActor.GetQuadMaterial();

	if (!Material)
	{
		return;
	}

	UDynamicMaterialInstance* MDI = Cast<UDynamicMaterialInstance>(Material);

	if (!MDI)
	{
		return;
	}

	const FPSDFileLayer* Layer = InQuadMeshActor.GetLayer();

	if (!Layer)
	{
		return;
	}

	UTexture* LayerTexture = Layer->Texture.LoadSynchronous();
	UTexture* MaskTexture = Layer->Mask.LoadSynchronous();
	const FIntPoint DocumentSize = PSDDocument->GetSize();
	const bool bIsMaskFullSize = Layer->IsMaskFullSize(DocumentSize);
	const FVector4 UVParams = bIsMaskFullSize ? FVector4(0.0, 0.0, 1.0, 1.0) : CalculateMaskParams(*Layer);
	const FVector4 CropParams = bIsMaskFullSize ? FVector4::Zero() : CalculateCropParams(*Layer);

	if (UDynamicMaterialModelDynamic* MaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(MDI->GetMaterialModelBase()))
	{
		for (const TPair<FName, TObjectPtr<UDMMaterialComponentDynamic>>& ComponentPair : MaterialModelDynamic->GetComponentMap())
		{
			if (UDMMaterialValueTextureDynamic* TextureValue = Cast<UDMMaterialValueTextureDynamic>(ComponentPair.Value))
			{
				if (UDMMaterialValue* ParentValue = TextureValue->GetParentValue())
				{
					SetTextureValue(TextureValue, ParentValue->GetMaterialParameterName(), LayerTexture, MaskTexture);
				}
			}
		}
	}
	else if (UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(MDI->GetMaterialModelBase()))
	{
		const TSet<TObjectPtr<UDMMaterialComponent>> Components = MaterialModel->GetRuntimeComponents();

		for (const TObjectPtr<UDMMaterialComponent>& Component : Components)
		{
			if (UDMMaterialValueTexture* TextureValue = Cast<UDMMaterialValueTexture>(Component))
			{
				SetTextureValue(TextureValue, TextureValue->GetMaterialParameterName(), LayerTexture, MaskTexture);
			}
		}
	}

	if (PSDDocument->WereLayersResizedOnImport())
	{
		return;
	}

	if (UDynamicMaterialModelDynamic* MaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(MDI->GetMaterialModelBase()))
	{
		for (const TPair<FName, TObjectPtr<UDMMaterialComponentDynamic>>& ComponentPair : MaterialModelDynamic->GetComponentMap())
		{
			if (UDMTextureUVDynamic* TextureUV = Cast<UDMTextureUVDynamic>(ComponentPair.Value))
			{
				if (UDMTextureUV* ParentUV = TextureUV->GetParentTextureUV())
				{
					const FName OffsetXParameterName = ParentUV->GetMaterialParameterName(UDMTextureUV::NAME_Offset, 0);

					if (OffsetXParameterName != UE::PSDImporterMaterialDesignerBridge::OpacityOffsetXParameterName)
					{
						continue;
					}

					SetTextureUV(TextureUV, UVParams);
				}
			}

			if (UDMMaterialValueFloat1Dynamic* FloatValue = Cast<UDMMaterialValueFloat1Dynamic>(ComponentPair.Value))
			{
				if (UDMMaterialValueFloat1* ParentValue = Cast<UDMMaterialValueFloat1>(FloatValue->GetParentValue()))
				{
					SetCropValue(FloatValue, ParentValue->GetMaterialParameterName(), CropParams);
				}
			}
		}
	}
	else if (UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(MDI->GetMaterialModelBase()))
	{
		const TSet<TObjectPtr<UDMMaterialComponent>> Components = MaterialModel->GetRuntimeComponents();

		for (const TObjectPtr<UDMMaterialComponent>& Component : Components)
		{
			if (UDMTextureUV* TextureUV = Cast<UDMTextureUV>(Component))
			{
				const FName OffsetXParameterName = TextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Offset, 0);

				if (OffsetXParameterName != UE::PSDImporterMaterialDesignerBridge::OpacityOffsetXParameterName)
				{
					continue;
				}

				SetTextureUV(TextureUV, UVParams);
			}

			if (UDMMaterialValueFloat1* FloatValue = Cast<UDMMaterialValueFloat1>(Component))
			{
				SetCropValue(FloatValue, FloatValue->GetMaterialParameterName(), CropParams);
			}
		}
	}
}
