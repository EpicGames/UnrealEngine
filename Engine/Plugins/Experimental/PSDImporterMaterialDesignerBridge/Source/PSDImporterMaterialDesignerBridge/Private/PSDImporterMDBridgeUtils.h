// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class APSDQuadMeshActor;

class FPSDImporterMDBridgeUtils
{
public:
	static void ResetTexture(APSDQuadMeshActor& InQuadMeshActor);
};
