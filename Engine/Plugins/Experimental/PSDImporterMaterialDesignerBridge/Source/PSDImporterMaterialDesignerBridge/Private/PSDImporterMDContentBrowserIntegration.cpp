// Copyright Epic Games, Inc. All Rights Reserved.

#include "PSDImporterMDContentBrowserIntegration.h"

#include "ContentBrowserModule.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "Factories/PSDImporterMDMaterialFactory.h"
#include "Factories/PSDImporterMDQuadsFactory.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Misc/MessageDialog.h"
#include "PSDDocument.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "PSDImporterMaterialDesignerContentBrowserIntegration"

FPSDImporterMaterialDesignerContentBrowserIntegration& FPSDImporterMaterialDesignerContentBrowserIntegration::Get()
{
	static FPSDImporterMaterialDesignerContentBrowserIntegration Object;
	return Object;
}

void FPSDImporterMaterialDesignerContentBrowserIntegration::Integrate()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	CBMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateRaw(this, &FPSDImporterMaterialDesignerContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu));
	ContentBrowserHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

void FPSDImporterMaterialDesignerContentBrowserIntegration::Disintegrate()
{
	if (!ContentBrowserHandle.IsValid())
	{
		return;
	}

	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule->GetAllAssetViewContextMenuExtenders();

		CBMenuExtenderDelegates.RemoveAll(
			[this](const FContentBrowserMenuExtender_SelectedAssets& InElement)
			{
				return InElement.GetHandle() == ContentBrowserHandle;
			}
		);
	}

	ContentBrowserHandle.Reset();
}

TSharedRef<FExtender> FPSDImporterMaterialDesignerContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	bool bHasDocument = false;

	for (const FAssetData& SelectedAsset : InSelectedAssets)
	{
		if (UClass* AssetClass = SelectedAsset.GetClass(EResolveClass::Yes))
		{
			if (AssetClass->IsChildOf<UPSDDocument>())
			{
				bHasDocument = true;
				break;
			}
		}
	}

	if (!bHasDocument)
	{
		return Extender;
	}

	Extender->AddMenuExtension(
		"GetAssetActions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateRaw(this, &FPSDImporterMaterialDesignerContentBrowserIntegration::CreateMenuEntries, InSelectedAssets)
	);

	return Extender;
}

void FPSDImporterMaterialDesignerContentBrowserIntegration::CreateMenuEntries(FMenuBuilder& InMenuBuilder, TArray<FAssetData> InSelectedAssets)
{
	InMenuBuilder.BeginSection(TEXT("PSDDocument"), LOCTEXT("PSDDocumentMaterialDesigner", "PSD Document (Material Designer)"));

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CreatePSDDocumentMateriaMaterialDesigner", "Create Material From PSD Document (Material Designer)"),
		LOCTEXT("CreatePSDDocumentMateriaMaterialDesignerTooltip", "Creates a material comprising of the layer stack in the PSD Document using a Material Designer Material."),
		FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()),
		FUIAction(FExecuteAction::CreateRaw(this, &FPSDImporterMaterialDesignerContentBrowserIntegration::CreatePSDMaterialMaterialDesigner, InSelectedAssets))
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CreatePSDDocumentQuadsMaterialDesignerInstance", "Create Quads From PSD Document (Material Designer (Instance))"),
		LOCTEXT("CreatePSDDocumentQuadsMaterialDesignerInstanceTooltip", "Creates a series of quads representing each layer in the PSD Document using instances of a Material Designer Material."),
		FSlateIconFinder::FindIconForClass(UStaticMesh::StaticClass()),
		FUIAction(FExecuteAction::CreateRaw(this, &FPSDImporterMaterialDesignerContentBrowserIntegration::CreatePSDQuadsMaterialDesigner, InSelectedAssets, EPSDImporterMaterialDesignerType::Instance))
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CreatePSDDocumentQuadsMaterialDesignerMaterial", "Create Quads From PSD Document (Material Designer (Material))"),
		LOCTEXT("CreatePSDDocumentQuadsMaterialDesignerMaterialTooltip", "Creates a series of quads representing each layer in the PSD Document using new Material Designer Materials."),
		FSlateIconFinder::FindIconForClass(UStaticMesh::StaticClass()),
		FUIAction(FExecuteAction::CreateRaw(this, &FPSDImporterMaterialDesignerContentBrowserIntegration::CreatePSDQuadsMaterialDesigner, InSelectedAssets, EPSDImporterMaterialDesignerType::Copy))
	);

	InMenuBuilder.EndSection();
}

void FPSDImporterMaterialDesignerContentBrowserIntegration::CreatePSDMaterialMaterialDesigner(TArray<FAssetData> InSelectedAssets)
{
	UPSDDocument* Document = nullptr;

	for (const FAssetData& AssetData : InSelectedAssets)
	{
		UClass* Class = AssetData.GetClass(EResolveClass::Yes);

		if (!Class || !Class->IsChildOf(UPSDDocument::StaticClass()))
		{
			continue;
		}

		Document = Cast<UPSDDocument>(AssetData.GetAsset());

		if (Document)
		{
			break;
		}
	}

	if (!Document)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CreatePSDMaterialDesignerMaterial", "Create PSD Material using Material Designer"));

	UPSDImporterMDMaterialFactory* Factory = NewObject<UPSDImporterMDMaterialFactory>();
	Factory->CreateMaterial(Document);
}

void FPSDImporterMaterialDesignerContentBrowserIntegration::CreatePSDQuadsMaterialDesigner(TArray<FAssetData> InSelectedAssets, EPSDImporterMaterialDesignerType InType)
{
	if (!GWorld)
	{
		return;
	}

	UPSDDocument* Document = nullptr;

	for (const FAssetData& AssetData : InSelectedAssets)
	{
		UClass* Class = AssetData.GetClass(EResolveClass::Yes);

		if (!Class || !Class->IsChildOf(UPSDDocument::StaticClass()))
		{
			continue;
		}

		Document = Cast<UPSDDocument>(AssetData.GetAsset());

		if (Document)
		{
			break;
		}
	}

	if (!Document)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CreatePSDMaterialDesignerQuads", "Create PSD Quads using Material Designer"));

	UPSDImporterMDQuadsFactory* QuadsFactory = GetMutableDefault<UPSDImporterMDQuadsFactory>();

	if (APSDQuadActor* QuadActor = QuadsFactory->CreateQuadActor(*GWorld, *Document))
	{
		QuadsFactory->CreateQuads(*QuadActor, InType);
	}
}

#undef LOCTEXT_NAMESPACE
