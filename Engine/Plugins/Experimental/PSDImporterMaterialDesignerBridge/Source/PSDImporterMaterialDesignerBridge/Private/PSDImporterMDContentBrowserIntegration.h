// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

class FExtender;
class FMenuBuilder;
enum class EPSDImporterMaterialDesignerType : uint8;

class FPSDImporterMaterialDesignerContentBrowserIntegration
{
public:
	static FPSDImporterMaterialDesignerContentBrowserIntegration& Get();

	void Integrate();

	void Disintegrate();

protected:
	TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets);

	void CreateMenuEntries(FMenuBuilder& InMenuBuilder, TArray<FAssetData> InSelectedAssets);

	void CreatePSDMaterialMaterialDesigner(TArray<FAssetData> InSelectedAssets);

	void CreatePSDQuadsMaterialDesigner(TArray<FAssetData> InSelectedAssets, EPSDImporterMaterialDesignerType InType);

	FDelegateHandle ContentBrowserHandle;
};
