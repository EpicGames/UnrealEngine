// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::PSDImporterMaterialDesignerBridge
{
	constexpr const TCHAR* TextureEmissiveParameterName = TEXT("LayerEmissiveTexture");
	constexpr const TCHAR* TextureOpacityParameterName = TEXT("LayerOpacityTexture");
	constexpr const TCHAR* OpacityOffsetXParameterName = TEXT("LayerOpacityOffsetX");
	constexpr const TCHAR* OpacityOffsetYParameterName = TEXT("LayerOpacityOffsetX");
	constexpr const TCHAR* OpacityTilingXParameterName = TEXT("LayerOpacityTilingX");
	constexpr const TCHAR* OpacityTilingYParameterName = TEXT("LayerOpacityTilingX");
	constexpr const TCHAR* OpacityCropLeft = TEXT("CropLeft");
	constexpr const TCHAR* OpacityCropRight = TEXT("CropRight");
	constexpr const TCHAR* OpacityCropTop = TEXT("CropTop");
	constexpr const TCHAR* OpacityCropBottom = TEXT("CropBottom");
}
